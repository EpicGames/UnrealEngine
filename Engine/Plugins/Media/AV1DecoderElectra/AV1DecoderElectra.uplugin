{
  "FileVersion": 3,
  "Version": 1,
  "VersionName": "1.0",
  "FriendlyName": "AV1 software decoder for Electra",
  "Description": "Enables decoding of AV1 encoded videos with the Electra media player",
  "CreatedBy": "Epic Games, Inc.",
  "CreatedByURL": "https://epicgames.com",
  "Category": "Media Players",
  "EnabledByDefault": false,
  "CanContainContent": false,
  "IsBetaVersion": true,
  "IsExperimentalVersion": true,
  "Installed": false,
  "Modules": [
    {
      "Name": "AV1DecoderElectra",
      "Type": "Runtime",
      "LoadingPhase": "PostEngineInit",
      "PlatformAllowList": [ "Win64", "Linux" ],
      "TargetDenyList": [ "Server" ]
    }
  ],
  "Plugins": [
    {
      "Name": "ElectraCodecs",
      "Enabled": true
    }
  ]
}
