// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;

namespace UnrealBuildTool.Rules
{
	public class AV1DecoderElectra : ModuleRules
	{
		public AV1DecoderElectra(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"ElectraBase",
                    "ElectraSamples",
					"ElectraCodecFactory",
					"ElectraDecoders",
					"LibDav1d"
                });

			if (Target.Platform.IsInGroup(UnrealPlatformGroup.Windows))
			{
				PublicDependencyModuleNames.Add("DirectX");
				AddEngineThirdPartyPrivateStaticDependencies(Target, "DX12");

				if (Target.Platform.IsInGroup(UnrealPlatformGroup.Windows) && Target.WindowsPlatform.Architecture != UnrealArch.Arm64)
				{
					PublicAdditionalLibraries.AddRange(new string[] {
						Path.Combine(Target.WindowsPlatform.DirectXLibDir, "dxerr.lib"),
					});
				}
			}
			else if (Target.Platform.IsInGroup(UnrealPlatformGroup.Unix))
			{
				// TBD
			}
		}
	}
}
