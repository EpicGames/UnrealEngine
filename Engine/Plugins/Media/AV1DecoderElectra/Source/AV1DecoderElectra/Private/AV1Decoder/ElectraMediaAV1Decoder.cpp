// Copyright Epic Games, Inc. All Rights Reserved.

#include "ElectraMediaAV1Decoder.h"
#include "AV1DecoderElectraModule.h"
#include "IElectraCodecRegistry.h"
#include "IElectraCodecFactory.h"
#include "Misc/ScopeLock.h"
#include "Templates/SharedPointer.h"
#include "Features/IModularFeatures.h"
#include "Features/IModularFeature.h"
#include "IElectraCodecFactoryModule.h"
#include "IElectraDecoderFeaturesAndOptions.h"
#include "IElectraDecoder.h"
#include "IElectraDecoderOutputVideo.h"
#include "ElectraDecodersUtils.h"
#include "Utils/AOMedia/ElectraBitstreamProcessor_AV1.h"
#include "Containers/Queue.h"
#include <Stats/Stats.h>


#define ENABLE_GPU_BUFFER_HELPER 1

#if ENABLE_GPU_BUFFER_HELPER
#include "ElectraDecodersPlatformResources.h"
#include COMPILED_PLATFORM_HEADER(ElectraTextureSampleGPUBufferHelper.h)
#else
#define ELECTRA_MEDIAGPUBUFFER_DX12 0
#endif

#ifndef WITH_LIBDAV1D_AV1_DECODER
#define WITH_LIBDAV1D_AV1_DECODER 0
#endif

#if WITH_LIBDAV1D_AV1_DECODER
#include <dav1d.h>
#endif

/*********************************************************************************************************************/


/*********************************************************************************************************************/

DECLARE_CYCLE_STAT(TEXT("ElectraDecoder ConvertOutput"), STAT_ElectraDecoder_ConvertOutputDav1d, STATGROUP_Media);

/*********************************************************************************************************************/

#define ERRCODE_INTERNAL_NO_ERROR							0
#define ERRCODE_INTERNAL_ALREADY_CLOSED						1
#define ERRCODE_INTERNAL_COULD_NOT_CREATE_DECODER			2
#define ERRCODE_INTERNAL_FAILED_TO_DECODE_INPUT				3
#define ERRCODE_INTERNAL_FAILED_TO_FLUSH_DECODER			4
#define ERRCODE_INTERNAL_FAILED_TO_CONVERT_OUTPUT_SAMPLE	5

/*********************************************************************************************************************/
/*********************************************************************************************************************/
/*********************************************************************************************************************/

#if WITH_LIBDAV1D_AV1_DECODER

class FVideoDecoderAV1Electra;


class FVideoDecoderOutputAV1Electra : public IElectraDecoderVideoOutput, public IElectraDecoderVideoOutputImageBuffers
{
public:
	virtual ~FVideoDecoderOutputAV1Electra()
	{ }

	FTimespan GetPTS() const override
	{ return PTS; }
	uint64 GetUserValue() const override
	{ return UserValue; }

	EOutputType GetOutputType() const
	{ return EOutputType::Output; }
	int32 GetWidth() const override
	{ return Width - Crop.Left - Crop.Right; }
	int32 GetHeight() const override
	{ return Height - Crop.Top - Crop.Bottom; }
	int32 GetDecodedWidth() const override
	{ return DecodedWidth; }
	int32 GetDecodedHeight() const override
	{ return DecodedHeight; }
	FElectraVideoDecoderOutputCropValues GetCropValues() const override
	{ return Crop; }
	int32 GetAspectRatioW() const override
	{ return AspectW; }
	int32 GetAspectRatioH() const override
	{ return AspectH; }
	int32 GetFrameRateNumerator() const override
	{ return FrameRateN; }
	int32 GetFrameRateDenominator() const override
	{ return FrameRateD; }
	int32 GetNumberOfBits() const override
	{ return NumBits; }
	void GetExtraValues(TMap<FString, FVariant>& OutExtraValues) const override
	{ OutExtraValues = ExtraValues; }
	void* GetPlatformOutputHandle(EElectraDecoderPlatformOutputHandleType InTypeOfHandle) const override
	{
		if (InTypeOfHandle == EElectraDecoderPlatformOutputHandleType::ImageBuffers)
		{
			return static_cast<IElectraDecoderVideoOutputImageBuffers*>(const_cast<FVideoDecoderOutputAV1Electra*>(this));
		}
		return nullptr;
	}
	IElectraDecoderVideoOutput::EImageCopyResult CopyPlatformImage(IElectraDecoderVideoOutputCopyResources* InCopyResources) const override
	{ return IElectraDecoderVideoOutput::EImageCopyResult::NotSupported; }

	// Methods from IElectraDecoderVideoOutputImageBuffers
	uint32 GetCodec4CC() const override
	{ return Codec4CC; }
	int32 GetNumberOfBuffers() const override
	{
		return NumBuffers;
	}
	TSharedPtr<TArray64<uint8>, ESPMode::ThreadSafe> GetBufferDataByIndex(int32 InBufferIndex) const override
	{
		if (InBufferIndex == 0)
		{
			return ColorBuffer;
		}
		return nullptr;
	}
	void* GetBufferTextureByIndex(int32 InBufferIndex) const override
	{
#if ELECTRA_MEDIAGPUBUFFER_DX12
		if (InBufferIndex == 0)
		{
			return GPUBuffer.Resource.GetReference();
		}
#endif
		return nullptr;
	}
	virtual bool GetBufferTextureSyncByIndex(int32 InBufferIndex, FElectraDecoderOutputSync& SyncObject) const override
	{
#if ELECTRA_MEDIAGPUBUFFER_DX12
		if (InBufferIndex == 0)
		{
			SyncObject = { GPUBuffer.Fence.GetReference(), GPUBuffer.FenceValue };
			return true;
		}
#endif
		return false;
	}
	EPixelFormat GetBufferFormatByIndex(int32 InBufferIndex) const override
	{
		if (InBufferIndex == 0)
		{
			return ColorBufferFormat;
		}
		return EPixelFormat::PF_Unknown;
	}
	EElectraTextureSamplePixelEncoding GetBufferEncodingByIndex(int32 InBufferIndex) const override
	{
		if (InBufferIndex == 0)
		{
			return ColorBufferEncoding;
		}
		return EElectraTextureSamplePixelEncoding::Native;
	}
	int32 GetBufferPitchByIndex(int32 InBufferIndex) const override
	{
		if (InBufferIndex == 0)
		{
			return ColorPitch;
		}
		return 0;
	}

public:
	FTimespan PTS;
	uint64 UserValue = 0;

	FElectraVideoDecoderOutputCropValues Crop;
	int32 Width = 0;
	int32 Height = 0;
	int32 DecodedWidth = 0;
	int32 DecodedHeight = 0;
	int32 NumBits = 0;
	int32 AspectW = 1;
	int32 AspectH = 1;
	int32 FrameRateN = 0;
	int32 FrameRateD = 0;
	TMap<FString, FVariant> ExtraValues;

	uint32 Codec4CC = 0;
	int32 NumBuffers = 0;
	TSharedPtr<TArray64<uint8>, ESPMode::ThreadSafe> ColorBuffer;
	EPixelFormat ColorBufferFormat;
	EElectraTextureSamplePixelEncoding ColorBufferEncoding;
	int32 ColorPitch = 0;
#if ELECTRA_MEDIAGPUBUFFER_DX12
	FElectraMediaDecoderOutputBufferPool_DX12::FOutputData GPUBuffer;
#endif
};


class FVideoDecoderAV1Electra : public IElectraDecoder
{
public:
	static void GetConfigurationOptions(TMap<FString, FVariant>& OutOptions)
	{
		OutOptions.Emplace(IElectraDecoderFeature::MinimumNumberOfOutputFrames, FVariant((int32)5));
		OutOptions.Emplace(IElectraDecoderFeature::IsAdaptive, FVariant(false));
	}

	FVideoDecoderAV1Electra(const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions);
	virtual ~FVideoDecoderAV1Electra();

	IElectraDecoder::EType GetType() const override
	{ return IElectraDecoder::EType::Video; }

	void GetFeatures(TMap<FString, FVariant>& OutFeatures) const override;

	FError GetError() const override;

	void Close() override;
	IElectraDecoder::ECSDCompatibility IsCompatibleWith(const TMap<FString, FVariant>& CSDAndAdditionalOptions) override;
	bool ResetToCleanStart() override;

	EDecoderError DecodeAccessUnit(const FInputAccessUnit& InInputAccessUnit, const TMap<FString, FVariant>& InAdditionalOptions) override;
	EDecoderError SendEndOfData() override;
	EDecoderError Flush() override;
	EOutputStatus HaveOutput() override;
	TSharedPtr<IElectraDecoderOutput, ESPMode::ThreadSafe> GetOutput() override;

	TSharedPtr<IElectraDecoderBitstreamProcessor, ESPMode::ThreadSafe> CreateBitstreamProcessor() override
	{
		TMap<FString, FVariant> DecoderFeatures;
		GetFeatures(DecoderFeatures);
		return FElectraDecoderBitstreamProcessorAV1::Create(DecoderFeatures, InitialCodecFormat, InitialCreationOptions);
	}

	void Suspend() override
	{ }
	void Resume() override
	{ }

private:
	struct FDecoderInput
	{
		~FDecoderInput()
		{
			FMemory::Free(InputDataCopy);
		}
		FInputAccessUnit AccessUnit;
		TMap<FString, FVariant> AdditionalOptions;
		uint8* InputDataCopy = nullptr;
		int32 InputDataSize = 0;
	};

	enum class EDecodeState
	{
		Decoding,
		Draining
	};

	static constexpr uint32 Make4CC(const uint8 A, const uint8 B, const uint8 C, const uint8 D)
	{
		return (static_cast<uint32>(A) << 24) | (static_cast<uint32>(B) << 16) | (static_cast<uint32>(C) << 8) | static_cast<uint32>(D);
	}

	bool PostError(int32 ApiReturnValue, FString Message, int32 Code);

	bool InternalDecoderCreate(const FInputAccessUnit& InInputAccessUnit, const TMap<FString, FVariant>& InAdditionalOptions);
	void InternalDecoderDestroy();

	IElectraDecoder::EDecoderError DecodeNextPending();

	enum class EConvertResult
	{
		Success,
		Failure,
		GotEOS
	};
	EConvertResult ConvertDecoderOutput(Dav1dPicture* InDecodedImage);
	bool ConvertDecodedImageToNV12orP010(FVideoDecoderOutputAV1Electra* InNewOutput, const Dav1dPicture* InDecodedImage, void* PlatformDevice, int32 PlatformDeviceVersion) const;

	Electra::FCodecTypeFormat InitialCodecFormat;
	TMap<FString, FVariant> InitialCreationOptions;

	TQueue<TSharedPtr<FDecoderInput, ESPMode::NotThreadSafe>, EQueueMode::Spsc> PendingDecoderInput;
	TArray<TSharedPtr<FDecoderInput, ESPMode::NotThreadSafe>> InDecoderInput;
	TSharedPtr<FVideoDecoderOutputAV1Electra, ESPMode::ThreadSafe> CurrentOutput;
	IElectraDecoder::FError LastError;
	EDecodeState DecodeState = EDecodeState::Decoding;

	struct FDecoderStruct
	{
		//TArray<uint8> ConfigOBUs;
		Dav1dSettings Settings;
		Dav1dContext* Handle = nullptr;
	};
	FDecoderStruct Decoder;

	uint32 MaxWidth;
	uint32 MaxHeight;
	uint32 MaxOutputBuffers;
#if ELECTRA_MEDIAGPUBUFFER_DX12
	mutable TSharedPtr<FElectraMediaDecoderOutputBufferPool_DX12, ESPMode::ThreadSafe> D3D12ResourcePool;
#endif
};

/*********************************************************************************************************************/
/*********************************************************************************************************************/
/*********************************************************************************************************************/

class FAV1VideoDecoderElectraFactory : public IElectraCodecFactory, public IElectraCodecFactory::IProviderInformation, public IElectraCodecModularFeature, public TSharedFromThis<FAV1VideoDecoderElectraFactory, ESPMode::ThreadSafe>
{
public:
	virtual ~FAV1VideoDecoderElectraFactory()
	{ }

	FAV1VideoDecoderElectraFactory()
	{
		LibVersion = ANSI_TO_TCHAR(dav1d_version());
	}

	void GetListOfFactories(TArray<TWeakPtr<IElectraCodecFactory, ESPMode::ThreadSafe>>& OutCodecFactories) override
	{
		OutCodecFactories.Add(AsShared());
	}

	void GetConfigurationOptions(TMap<FString, FVariant>& OutOptions) const override
	{
		FVideoDecoderAV1Electra::GetConfigurationOptions(OutOptions);
	}

	int32 SupportsDecoding(TMap<FString, FVariant>& OutFormatInfo, const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions) const override
	{
		if (InCodecFormat.Type != Electra::FCodecTypeFormat::EType::Video || !InCodecFormat.Properties.IsType<Electra::FCodecTypeFormat::FVideo>())
		{
			return 0;
		}
		const Electra::FCodecTypeFormat::FVideo& vi = InCodecFormat.Properties.Get<Electra::FCodecTypeFormat::FVideo>();
		if (InCodecFormat.FourCC == ElectraDecodersUtil::Make4CC('a','v','0','1'))
		{
			return vi.BitDepth == 8 || vi.BitDepth == 10 || vi.BitDepth == 12 ? 1 : 0;
		}
		return 0;
	}

	TSharedPtr<IElectraDecoder, ESPMode::ThreadSafe> CreateDecoder(const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions) override
	{
		return InCodecFormat.Properties.IsType<Electra::FCodecTypeFormat::FVideo>() ? MakeShared<FVideoDecoderAV1Electra, ESPMode::ThreadSafe>(InCodecFormat, InAdditionalOptions) : TSharedPtr<IElectraDecoder, ESPMode::ThreadSafe>();
	}


	const IProviderInformation& GetProviderInformation() const override
	{ return *this; }
	FString GetName() const override
	{ return FString(TEXT("libdav1d")); }
	FString GetVersion() const override
	{ return LibVersion; }
	FString GetImplementation() const override
	{ return(FString()); }
	FString GetVendor() const override
	{ return(FString()); }

	FString LibVersion;
	static TSharedPtr<FAV1VideoDecoderElectraFactory, ESPMode::ThreadSafe> Self;
};
TSharedPtr<FAV1VideoDecoderElectraFactory, ESPMode::ThreadSafe> FAV1VideoDecoderElectraFactory::Self;

#endif // WITH_LIBDAV1D_AV1_DECODER

/*********************************************************************************************************************/
/*********************************************************************************************************************/
/*********************************************************************************************************************/

void FElectraMediaAV1Decoder::Startup()
{
#if WITH_LIBDAV1D_AV1_DECODER
	// Make sure the codec factory module has been loaded.
	FModuleManager::Get().LoadModule(TEXT("ElectraCodecFactory"));

	// Create an instance of the factory, which is also the modular feature.
	check(!FAV1VideoDecoderElectraFactory::Self.IsValid());
	FAV1VideoDecoderElectraFactory::Self = MakeShared<FAV1VideoDecoderElectraFactory, ESPMode::ThreadSafe>();
	// Register as modular feature.
	IModularFeatures::Get().RegisterModularFeature(IElectraCodecFactoryModule::GetModularFeatureName(), FAV1VideoDecoderElectraFactory::Self.Get());
#endif
}

void FElectraMediaAV1Decoder::Shutdown()
{
#if WITH_LIBDAV1D_AV1_DECODER
	IModularFeatures::Get().UnregisterModularFeature(IElectraCodecFactoryModule::GetModularFeatureName(), FAV1VideoDecoderElectraFactory::Self.Get());
	FAV1VideoDecoderElectraFactory::Self.Reset();
#endif
}

/*********************************************************************************************************************/
/*********************************************************************************************************************/
/*********************************************************************************************************************/
#if WITH_LIBDAV1D_AV1_DECODER

FVideoDecoderAV1Electra::FVideoDecoderAV1Electra(const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions)
	: InitialCodecFormat(InCodecFormat)
	, InitialCreationOptions(InAdditionalOptions)
{
	const Electra::FCodecTypeFormat::FVideo& vid(InitialCodecFormat.Properties.Get<Electra::FCodecTypeFormat::FVideo>());

	MaxWidth = (uint32)Align(ElectraDecodersUtil::GetVariantValueSafeU64(InitialCreationOptions, TEXT("max_width"), 1920), 2);
	MaxHeight = (uint32)Align(ElectraDecodersUtil::GetVariantValueSafeU64(InitialCreationOptions, TEXT("max_height"), 1080), 2);
	MaxOutputBuffers = (uint32)ElectraDecodersUtil::GetVariantValueSafeU64(InitialCreationOptions, TEXT("max_output_buffers"), 5);
#if ELECTRA_MEDIAGPUBUFFER_DX12
	MaxOutputBuffers += kElectraDecoderPipelineExtraFrames;
#endif
}

FVideoDecoderAV1Electra::~FVideoDecoderAV1Electra()
{
	// Close() must have been called already!
	check(LastError.Code == ERRCODE_INTERNAL_ALREADY_CLOSED);
	// We do it nonetheless...
	Close();
}

void FVideoDecoderAV1Electra::GetFeatures(TMap<FString, FVariant>& OutFeatures) const
{
	GetConfigurationOptions(OutFeatures);
}

IElectraDecoder::FError FVideoDecoderAV1Electra::GetError() const
{
	return LastError;
}

bool FVideoDecoderAV1Electra::PostError(int32 ApiReturnValue, FString Message, int32 Code)
{
	LastError.Code = Code;
	LastError.SdkCode = ApiReturnValue;
	LastError.Message = MoveTemp(Message);
	return false;
}

void FVideoDecoderAV1Electra::Close()
{
	ResetToCleanStart();
	// Set the error state that all subsequent calls will fail.
	PostError(0, TEXT("Already closed"), ERRCODE_INTERNAL_ALREADY_CLOSED);
}

IElectraDecoder::ECSDCompatibility FVideoDecoderAV1Electra::IsCompatibleWith(const TMap<FString, FVariant>& CSDAndAdditionalOptions)
{
	if (!Decoder.Handle)
	{
		return IElectraDecoder::ECSDCompatibility::Compatible;
	}
	return IElectraDecoder::ECSDCompatibility::Drain;
}

bool FVideoDecoderAV1Electra::ResetToCleanStart()
{
	InternalDecoderDestroy();

	PendingDecoderInput.Empty();
	InDecoderInput.Empty();
	CurrentOutput.Reset();
	DecodeState = EDecodeState::Decoding;

	return !LastError.IsSet();
}

IElectraDecoder::EDecoderError FVideoDecoderAV1Electra::DecodeAccessUnit(const FInputAccessUnit& InInputAccessUnit, const TMap<FString, FVariant>& InAdditionalOptions)
{
	// If already in error do nothing!
	if (LastError.IsSet())
	{
		return IElectraDecoder::EDecoderError::Error;
	}

	// Can not feed new input until draining has finished.
	if (DecodeState == EDecodeState::Draining)
	{
		return IElectraDecoder::EDecoderError::EndOfData;
	}

	// If we still have pending input we do not want anything new right now.
	if (!PendingDecoderInput.IsEmpty())
	{
		return IElectraDecoder::EDecoderError::NoBuffer;
	}

	// If there is pending output it is very likely that decoding this access unit would also generate output.
	// Since that would result in loss of the pending output we return now.
	if (CurrentOutput.IsValid())
	{
		return IElectraDecoder::EDecoderError::NoBuffer;
	}

#if ELECTRA_MEDIAGPUBUFFER_DX12
	// If we will create a new resource pool or we have still buffers in an existing one, we can proceed, else we'd have no resources to output the data
	if (D3D12ResourcePool.IsValid() && !D3D12ResourcePool->BufferAvailable())
	{
		return IElectraDecoder::EDecoderError::NoBuffer;
	}
#endif

	// CSD only buffer is not handled at the moment.
	check((InInputAccessUnit.Flags & EElectraDecoderFlags::InitCSDOnly) == EElectraDecoderFlags::None);

	// Create decoder transform if necessary.
	if (!Decoder.Handle && !InternalDecoderCreate(InInputAccessUnit, InAdditionalOptions))
	{
		return IElectraDecoder::EDecoderError::Error;
	}

	// Decode data. This immediately produces a new output frame.
	if (InInputAccessUnit.Data && InInputAccessUnit.DataSize)
	{
		TSharedPtr<FDecoderInput, ESPMode::NotThreadSafe> In(new FDecoderInput);
		In->AdditionalOptions = InAdditionalOptions;
		In->AccessUnit = InInputAccessUnit;
		In->InputDataSize = InInputAccessUnit.DataSize;
		In->InputDataCopy = (uint8*)FMemory::Malloc(InInputAccessUnit.DataSize);
		FMemory::Memcpy(In->InputDataCopy, InInputAccessUnit.Data, InInputAccessUnit.DataSize);
		In->AccessUnit.Data = nullptr;
		In->AccessUnit.DataSize = 0;
		PendingDecoderInput.Enqueue(MoveTemp(In));
		return DecodeNextPending();
	}
	return IElectraDecoder::EDecoderError::None;
}

IElectraDecoder::EDecoderError FVideoDecoderAV1Electra::DecodeNextPending()
{
	TSharedPtr<FDecoderInput, ESPMode::NotThreadSafe> In;
	if (!PendingDecoderInput.Dequeue(In))
	{
		return IElectraDecoder::EDecoderError::None;
	}

	// Send data to the decoder
	Dav1dData DecData;
	FMemory::Memzero(DecData);
	DecData.data = In->InputDataCopy;
	DecData.sz = In->InputDataSize;
	DecData.m.user_data.data = reinterpret_cast<const uint8_t *>(In.Get());
	int32 Result = dav1d_send_data(Decoder.Handle, &DecData);

	if (Result == DAV1D_ERR(EAGAIN))
	{
		return IElectraDecoder::EDecoderError::NoBuffer;
	}
	else if (Result != 0)
	{
		PostError(Result, TEXT("Failed to decode video decoder input"), ERRCODE_INTERNAL_FAILED_TO_DECODE_INPUT);
		return IElectraDecoder::EDecoderError::Error;
	}

	// Add to the list of inputs passed to the decoder.
	InDecoderInput.Emplace(MoveTemp(In));

	// Did we produce a new frame?
	Dav1dPicture OutPic;
	FMemory::Memzero(OutPic);
	Result = dav1d_get_picture(Decoder.Handle, &OutPic);
	// Got output?
	if (Result == 0)
	{
		EConvertResult ConvResult = ConvertDecoderOutput(&OutPic);
		// No matter if conversion was successful. We no longer need the decoded image.
		dav1d_picture_unref(&OutPic);
		if (ConvResult == EConvertResult::Failure)
		{
			return IElectraDecoder::EDecoderError::Error;
		}
		return IElectraDecoder::EDecoderError::None;
	}
	else if (Result == DAV1D_ERR(EAGAIN))
	{
		// No output yet, need more input.
		return IElectraDecoder::EDecoderError::None;
	}
	else
	{
		PostError(Result, TEXT("Failed to decode video decoder input"), ERRCODE_INTERNAL_FAILED_TO_DECODE_INPUT);
		return IElectraDecoder::EDecoderError::Error;
	}
}

IElectraDecoder::EDecoderError FVideoDecoderAV1Electra::SendEndOfData()
{
	// If already in error do nothing!
	if (LastError.IsSet())
	{
		return IElectraDecoder::EDecoderError::Error;
	}

	// Already draining?
	if (DecodeState == EDecodeState::Draining)
	{
		return IElectraDecoder::EDecoderError::EndOfData;
	}

	// If there is a transform send an end-of-stream and drain message.
	if (Decoder.Handle)
	{
		DecodeState = EDecodeState::Draining;
	}
	return IElectraDecoder::EDecoderError::None;
}

IElectraDecoder::EDecoderError FVideoDecoderAV1Electra::Flush()
{
	// If already in error do nothing!
	if (LastError.IsSet())
	{
		return IElectraDecoder::EDecoderError::Error;
	}
	if (Decoder.Handle)
	{
		InternalDecoderDestroy();
		DecodeState = EDecodeState::Decoding;
		PendingDecoderInput.Empty();
		InDecoderInput.Empty();
		CurrentOutput.Reset();
	}
	return IElectraDecoder::EDecoderError::None;
}

IElectraDecoder::EOutputStatus FVideoDecoderAV1Electra::HaveOutput()
{
	// If already in error do nothing!
	if (LastError.IsSet())
	{
		return IElectraDecoder::EOutputStatus::Error;
	}
	// Have output?
	if (CurrentOutput.IsValid())
	{
		return IElectraDecoder::EOutputStatus::Available;
	}

	// See if there is any additional pending output.
	if (Decoder.Handle)
	{
		Dav1dPicture OutPic;
		FMemory::Memzero(OutPic);
		int32 Result = dav1d_get_picture(Decoder.Handle, &OutPic);
		// If there is a frame, pull it.
		if (Result == 0)
		{
			EConvertResult ConvResult = ConvertDecoderOutput(&OutPic);
			// No matter if conversion was successful. We no longer need the decoded image.
			dav1d_picture_unref(&OutPic);
			if (ConvResult == EConvertResult::Failure)
			{
				return IElectraDecoder::EOutputStatus::Error;
			}
			else if (ConvResult == EConvertResult::Success && CurrentOutput.IsValid())
			{
				return IElectraDecoder::EOutputStatus::Available;
			}
		}
		// If no frame available and we are draining, we are at EOS.
		else if (DecodeState == EDecodeState::Draining && Result == DAV1D_ERR(EAGAIN))
		{
			DecodeState = EDecodeState::Decoding;
			PendingDecoderInput.Empty();
			InDecoderInput.Empty();
			return IElectraDecoder::EOutputStatus::EndOfData;
		}

	}
	// Decode any pending input now.
	if (Decoder.Handle && !PendingDecoderInput.IsEmpty())
	{
		switch(DecodeNextPending())
		{
			case IElectraDecoder::EDecoderError::NoBuffer:
			{
				return IElectraDecoder::EOutputStatus::TryAgainLater;
			}
			case IElectraDecoder::EDecoderError::None:
			{
				if (CurrentOutput.IsValid())
				{
					return IElectraDecoder::EOutputStatus::Available;
				}
				break;
			}
			case IElectraDecoder::EDecoderError::Error:
			{
				return IElectraDecoder::EOutputStatus::Error;
			}
			default:
			{
				break;
			}
		}
	}
	return IElectraDecoder::EOutputStatus::NeedInput;
}

TSharedPtr<IElectraDecoderOutput, ESPMode::ThreadSafe> FVideoDecoderAV1Electra::GetOutput()
{
	TSharedPtr<IElectraDecoderOutput, ESPMode::ThreadSafe> Out = CurrentOutput;
	CurrentOutput.Reset();
	return Out;
}

bool FVideoDecoderAV1Electra::InternalDecoderCreate(const FInputAccessUnit& InInputAccessUnit, const TMap<FString, FVariant>& InAdditionalOptions)
{
	FMemory::Memzero(Decoder.Settings);
	Decoder.Handle = nullptr;
	dav1d_default_settings(&Decoder.Settings);
	int32 Result = dav1d_open(&Decoder.Handle, &Decoder.Settings);
	if (Result)
	{
		return PostError(Result, TEXT("Failed to create decoder"), ERRCODE_INTERNAL_COULD_NOT_CREATE_DECODER);
	}
	return true;
}

void FVideoDecoderAV1Electra::InternalDecoderDestroy()
{
	if (Decoder.Handle)
	{
		dav1d_flush(Decoder.Handle);
		dav1d_close(&Decoder.Handle);
		Decoder.Handle = nullptr;
	}
}


FVideoDecoderAV1Electra::EConvertResult FVideoDecoderAV1Electra::ConvertDecoderOutput(Dav1dPicture* InDecodedImage)
{
	SCOPE_CYCLE_COUNTER(STAT_ElectraDecoder_ConvertOutputDav1d);

	// Find the input corresponding to this output.
	TSharedPtr<FDecoderInput, ESPMode::NotThreadSafe> In;
	for(int32 nInDec=0; nInDec<InDecoderInput.Num(); ++nInDec)
	{
		if (reinterpret_cast<const uint8_t *>(InDecoderInput[nInDec].Get()) == InDecodedImage->m.user_data.data)
		{
			In = InDecoderInput[nInDec];
			InDecoderInput.RemoveAt(nInDec);
			break;
		}
	}
	if (!In.IsValid())
	{
		PostError(0, TEXT("There is no matching decoder input for the decoded output!"), ERRCODE_INTERNAL_FAILED_TO_CONVERT_OUTPUT_SAMPLE);
		return EConvertResult::Failure;
	}

	TSharedPtr<FVideoDecoderOutputAV1Electra, ESPMode::ThreadSafe> NewOutput = MakeShared<FVideoDecoderOutputAV1Electra>();
	NewOutput->PTS = In->AccessUnit.PTS;
	NewOutput->UserValue = In->AccessUnit.UserValue;
	NewOutput->Width = InDecodedImage->p.w;
	NewOutput->Height = InDecodedImage->p.h;
	NewOutput->NumBits = InDecodedImage->p.bpc;
	NewOutput->AspectW = 1;
	NewOutput->AspectH = 1;

	void* PlatformDevice = nullptr;
	int32 PlatformDeviceVersion = 0;
	bool bUseGPUBuffers = false;
#if ELECTRA_MEDIAGPUBUFFER_DX12
	FElectraDecodersPlatformResources::GetD3DDeviceAndVersion(&PlatformDevice, &PlatformDeviceVersion);
	bUseGPUBuffers = (PlatformDevice && PlatformDeviceVersion >= 12000);
#endif

	if (InDecodedImage->p.layout == Dav1dPixelLayout::DAV1D_PIXEL_LAYOUT_I420)
	{
		NewOutput->NumBuffers = 1;
		NewOutput->ColorBufferFormat = NewOutput->NumBits == 8 ? EPixelFormat::PF_NV12 : EPixelFormat::PF_P010;
		NewOutput->ColorBufferEncoding = EElectraTextureSamplePixelEncoding::Native;
		NewOutput->DecodedWidth = NewOutput->Width;
		NewOutput->DecodedHeight = bUseGPUBuffers ? NewOutput->Height : (NewOutput->Height * 3 / 2);
		if (!ConvertDecodedImageToNV12orP010(NewOutput.Get(), InDecodedImage, PlatformDevice, PlatformDeviceVersion))
		{
			PostError(0, FString::Printf(TEXT("Failed to convert decoded image")), ERRCODE_INTERNAL_FAILED_TO_CONVERT_OUTPUT_SAMPLE);
			return EConvertResult::Failure;
		}
		// dav1d returns the output in the lower bits, but the output pipe expects it in the upper bits. Post scale to compensate!
		if (NewOutput->NumBits == 10)
		{
			NewOutput->ExtraValues.Emplace(TEXT("pix_datascale"), FVariant(64.0));
		}
		else if (NewOutput->NumBits == 12)
		{
			NewOutput->ExtraValues.Emplace(TEXT("pix_datascale"), FVariant(16.0));
		}
	}
	else
	{
		PostError(0, FString::Printf(TEXT("Unsupported decoded image format (%d)"), InDecodedImage->p.layout), ERRCODE_INTERNAL_FAILED_TO_CONVERT_OUTPUT_SAMPLE);
		return EConvertResult::Failure;
	}

	NewOutput->ExtraValues.Emplace(TEXT("codec"), FVariant(TEXT("av1")));
	NewOutput->ExtraValues.Emplace(TEXT("platform"), FVariant(TEXT("generic")));
	CurrentOutput = MoveTemp(NewOutput);
	return EConvertResult::Success;
}

bool FVideoDecoderAV1Electra::ConvertDecodedImageToNV12orP010(FVideoDecoderOutputAV1Electra* InNewOutput, const Dav1dPicture* InDecodedImage, void* PlatformDevice, int32 PlatformDeviceVersion) const
{
	const bool bIsNV12 = InDecodedImage->p.bpc == 8;

	const int32 w = InDecodedImage->p.w;
	const int32 h = InDecodedImage->p.h;
	const int32 aw = Align(w, 2);
	const int32 ah = Align(h, 2);

	const uint8* SrcY = (const uint8*)InDecodedImage->data[0];
	const uint8* SrcU = (const uint8*)InDecodedImage->data[1];
	const uint8* SrcV = (const uint8*)InDecodedImage->data[2];
	const int32 PitchY = InDecodedImage->stride[0];
	const int32 PitchU = InDecodedImage->stride[1];
	const int32 PitchV = InDecodedImage->stride[1];
	if (!SrcY || !SrcU || !SrcV)
	{
		return false;
	}

	uint8* DstY = nullptr;
	uint8* DstUV = nullptr;
	uint32 DstPitch = 0;

	TSharedPtr<TArray64<uint8>, ESPMode::ThreadSafe>& OutNV12Buffer = InNewOutput->ColorBuffer;
#if ELECTRA_MEDIAGPUBUFFER_DX12
	if (!PlatformDevice || PlatformDeviceVersion < 12000)
#endif
	{
		uint32 Pitch = bIsNV12 ? aw : (aw * 2);
		int32 AllocSize = Pitch * (ah * 3 / 2);

#if ELECTRA_MEDIAGPUBUFFER_DX12
		InNewOutput->GPUBuffer.Resource = nullptr;
		InNewOutput->GPUBuffer.Fence = nullptr;
#endif

		OutNV12Buffer = MakeShared<TArray64<uint8>, ESPMode::ThreadSafe>();
		OutNV12Buffer->AddUninitialized(AllocSize);
		DstY = OutNV12Buffer->GetData();
		DstUV = DstY + Pitch * ah;
		DstPitch = Pitch;
	}
#if ELECTRA_MEDIAGPUBUFFER_DX12
	else
	{
		TRefCountPtr D3D12Device(static_cast<ID3D12Device*>(PlatformDevice));
		HRESULT Result;
		FString ResultMsg;

		OutNV12Buffer.Reset();
		InNewOutput->GPUBuffer.Resource = nullptr;

		// Create the resource pool as needed...
		if (!D3D12ResourcePool)
		{
			D3D12ResourcePool = MakeShared<FElectraMediaDecoderOutputBufferPool_DX12, ESPMode::ThreadSafe>(D3D12Device, MaxOutputBuffers, MaxWidth, MaxHeight * 3 / 2, bIsNV12 ? 1 : 2);
		}

		// Request resource and fence...
		uint32 BufferPitch;
		if (D3D12ResourcePool->AllocateOutputDataAsBuffer(Result, ResultMsg, InNewOutput->GPUBuffer, BufferPitch))
		{
			// Correct pitch to reflect resource's setup
			InNewOutput->ColorPitch = BufferPitch;

			// Map the buffer so we can get a CPU address to the WC configured buffer
			Result = InNewOutput->GPUBuffer.Resource->Map(0, nullptr, (void**)&DstY);
			if (FAILED(Result))
			{
				return false;
			}
			DstUV = DstY + BufferPitch * ah;
			DstPitch = BufferPitch;
		}
		else
		{
			return false;
		}
	}
#endif // ELECTRA_MEDIAGPUBUFFER_DX12

	InNewOutput->ColorPitch = DstPitch;

	if (bIsNV12)
	{
		for (int32 y = 0; y < h; ++y)
		{
			FMemory::Memcpy(DstY, SrcY, w);
			SrcY += PitchY;
			DstY += DstPitch;
		}
		for (int32 v = 0; v < h / 2; ++v)
		{
			uint8* DstUVLine = DstUV;
			for (int32 u = 0; u < w / 2; ++u)
			{
				*DstUVLine++ = SrcU[u];
				*DstUVLine++ = SrcV[u];
			}
			SrcU += PitchU;
			SrcV += PitchV;
			DstUV += DstPitch;
		}
	}
	else
	{
		// note: data is delivered in the lower 10-bits, but expected in the upper
		// -> instead of processing the data here, we provide a "data scale" attribute to be applied on conversion from YUV to RGB
		for (int32 y = 0; y < h; ++y)
		{
			FMemory::Memcpy(DstY, SrcY, int64(w) << 1);
			SrcY += PitchY;
			DstY += DstPitch;
		}
		for (int32 v = 0; v < h / 2; ++v)
		{
			uint16* DstUVLine = (uint16*)DstUV;
			const uint16* SrcU16 = (const uint16*)SrcU;
			const uint16* SrcV16 = (const uint16*)SrcV;
			for (int32 u = 0; u < w / 2; ++u)
			{
				*DstUVLine++ = SrcU16[u];
				*DstUVLine++ = SrcV16[u];
			}
			SrcU += PitchU;
			SrcV += PitchV;
			DstUV += DstPitch;
		}
	}

#if ELECTRA_MEDIAGPUBUFFER_DX12
	if (InNewOutput->GPUBuffer.Resource.IsValid())
	{
		InNewOutput->GPUBuffer.Resource->Unmap(0, nullptr);
		// To be compatible with implementations that might do the copy into the resource async, we also signal a fence
		// (strictly speaking we would not need to as this is all 100% synchronous and done before the GPU ever attempts to read from the resource)
		InNewOutput->GPUBuffer.Fence->Signal(InNewOutput->GPUBuffer.FenceValue);
	}
#endif
	return true;
}


#endif // WITH_LIBDAV1D_AV1_DECODER
