// Copyright Epic Games, Inc. All Rights Reserved.

#include "AV1DecoderElectraModule.h"
#include "AV1Decoder/ElectraMediaAV1Decoder.h"
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "AV1DecoderElectraModule"

DEFINE_LOG_CATEGORY(LogAV1ElectraDecoder);

class FAV1ElectraDecoderModule : public IModuleInterface
{
public:
	void StartupModule() override
	{
		FElectraMediaAV1Decoder::Startup();
	}

	void ShutdownModule() override
	{
		FElectraMediaAV1Decoder::Shutdown();
	}

	bool SupportsDynamicReloading() override
	{
		// Codec could still be in use
		return false;
	}
};

IMPLEMENT_MODULE(FAV1ElectraDecoderModule, AV1DecoderElectra);

#undef LOCTEXT_NAMESPACE
